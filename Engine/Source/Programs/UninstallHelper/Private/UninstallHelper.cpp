// Copyright Epic Games, Inc. All Rights Reserved.

#include "UninstallHelper.h"

#include "RequiredProgramMainCPPInclude.h"

IMPLEMENT_APPLICATION(UninstallHelper, "UninstallHelper");
DEFINE_LOG_CATEGORY(LogUninstallHelper);

namespace UninstallHelper
{

	struct FExecActionInfo
	{
		FString URL;
		FString Params;
	};

	struct FUninstallOptions
	{
		bool bShouldExecute = true;
		bool bAllowHarshReturnCodes = false;
		FString AppName = "";
		FString InstallDir = "";
		bool bDeletePersistentDownloadDir = true;
		bool bDeleteConfig = false;
		bool bInstallDirRelativeToExecutableDir = false;
		TArray<FString> AdditionalDeleteArtifacts = {};
		TArray<FExecActionInfo> AdditionalActions = {};
	};

	inline FString GetUserSavedDir(const FUninstallOptions& Options)
	{
		return FPaths::Combine(FPlatformProcess::UserSettingsDir(), Options.AppName, "Saved");
	}
	
	inline FString GetPersistentDownloadDir(const FUninstallOptions& Options)
	{
		return FPaths::Combine(GetUserSavedDir(Options), "PersistentDownloadDir", "");
	}
	
	inline FString GetConfigDir(const FUninstallOptions& Options)
	{
		return FPaths::Combine(GetUserSavedDir(Options), "Config", "");
	}

	FString ReplacePathTokens(const FString& Path, const FUninstallOptions& Options)
	{
		FString Result = Path;
		Result.ReplaceInline(TEXT("%LocalDataDir%"), FPlatformProcess::UserSettingsDir());
		Result.ReplaceInline(TEXT("%LocalSavedDir%"), *GetUserSavedDir(Options));
		Result.ReplaceInline(TEXT("%InstallDir%"), *Options.InstallDir);
		Result.ReplaceInline(TEXT("%GameName%"), *Options.AppName);
		return Result;
	}

	int32 ExecProcess(const FExecActionInfo& Action)
	{
		int32 Result;
		FPlatformProcess::ExecElevatedProcess(*Action.URL, *Action.Params, &Result);
		if (Result)
		{
			UE_LOGF(LogUninstallHelper, Error, "Action %ls %ls failed with exit code %d", *Action.URL, *Action.Params, Result);
		}
		return Result;
	}

	/**
	 * Runs the requested uninstall actions.
	 * @param Options The options to apply to this run.
	 * @return Whether or not all of the attempted actions were successful.
	 */
	EReturnCode Execute(const FUninstallOptions& Options)
	{
		// TODO: Decide whether an error should cause a graceful or hard fail
		EReturnCode Result = EReturnCode::Success;
		IFileManager& FileManager = IFileManager::Get();
		if (Options.bDeletePersistentDownloadDir)
		{
			const FString PersistentDownloadDir = GetPersistentDownloadDir(Options);
			if (FileManager.DirectoryExists(*PersistentDownloadDir))
			{
				UE_LOGF(LogUninstallHelper, Log, "Deleting the PersistentDownloadDir located at %ls", *PersistentDownloadDir);
				if (Options.bShouldExecute && !FileManager.DeleteDirectory(*PersistentDownloadDir, true, true))
				{
					Result = EReturnCode::DiskOperationFailed;
				}
			}
		}
		if (Options.bDeleteConfig)
		{
			const FString ConfigDir = GetConfigDir(Options);
			if (FileManager.DirectoryExists(*ConfigDir))
			{
				UE_LOGF(LogUninstallHelper, Log, "Deleting the Config directory located at %ls", *ConfigDir);
				if (Options.bShouldExecute && !FileManager.DeleteDirectory(*ConfigDir, true, true))
				{
					Result = EReturnCode::DiskOperationFailed;
				}
			}
		}
		// Delete artifacts
		for (const FString& Artifact : Options.AdditionalDeleteArtifacts)
		{
			FString CleanArtifactPath = ReplacePathTokens(Artifact, Options);
			if (FileManager.DirectoryExists(*CleanArtifactPath))
			{
				UE_LOGF(LogUninstallHelper, Log, "Deleting directory %ls", *CleanArtifactPath);
				if (Options.bShouldExecute && !FileManager.DeleteDirectory(*CleanArtifactPath, true, true))
				{
					Result = EReturnCode::DiskOperationFailed;
				}
			}
			else if (FileManager.FileExists(*CleanArtifactPath))
			{
				UE_LOGF(LogUninstallHelper, Log, "Deleting file %ls", *CleanArtifactPath);
				if (Options.bShouldExecute && !FileManager.Delete(*CleanArtifactPath, true, true))
				{
					Result = EReturnCode::DiskOperationFailed;
				}
			}
			else
			{
				UE_LOGF(LogUninstallHelper, Log, "Unable to find artifact '%ls' on the disk, skipping.", *CleanArtifactPath);
			}
		}
		// Execute other actions
		int32 LastProcessResult = 0;
		for (const FExecActionInfo& Action : Options.AdditionalActions)
		{
			UE_LOGF(LogUninstallHelper, Log, "Executing process %ls %ls...", *Action.URL, *Action.Params);
			if (Options.bShouldExecute && (LastProcessResult = ExecProcess(Action)) != 0)
			{
				UE_LOGF(LogUninstallHelper, Log, "Process %ls %ls failed with exit code %d", *Action.URL, *Action.Params, LastProcessResult);
				Result = EReturnCode::ExecActionFailed;
			}
		}

		if (Result != EReturnCode::Success)
		{
			UE_LOGF(LogUninstallHelper, Error, "UninstallHelper exited with code %d", static_cast<int32>(Result));
		}
		
		return Result;
	}
	
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	const FString CmdLine = FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, nullptr);
	if (!FCommandLine::Set(*CmdLine))
	{
		UE_LOGF(LogUninstallHelper, Error, "Failed to initialize command line. Exiting.");
		return static_cast<int32>(UninstallHelper::EReturnCode::ArgumentError);
	}
	
	UninstallHelper::FUninstallOptions Options;

	if (!FParse::Value(*CmdLine, TEXT("GameName="), Options.AppName))
	{
		UE_LOGF(LogUninstallHelper, Error, "An app name must be provided in order for this tool to run. Exiting.");
		return static_cast<int32>(UninstallHelper::EReturnCode::UnknownAppName);
	}

	if (!FParse::Value(*CmdLine, TEXT("InstallDir="), Options.InstallDir))
	{
		UE_LOGF(LogUninstallHelper, Error, "InstallDir must be provided in order for this tool to run. Exiting.");
		return static_cast<int32>(UninstallHelper::EReturnCode::InvalidInstallDir);
	}

	// InstallDir can optionally be treated as relative to the UninstallHelper executable directory instead of the current working directory.
	FParse::Bool(*CmdLine, TEXT("InstallDirRelativeToExecutableDir="), Options.bInstallDirRelativeToExecutableDir);
	if (Options.bInstallDirRelativeToExecutableDir && FPaths::IsRelative(Options.InstallDir))
	{
		const FString ExecutableDir = FPaths::GetPath(FPlatformProcess::ExecutablePath());
		Options.InstallDir = FPaths::ConvertRelativePathToFull(ExecutableDir, Options.InstallDir);
	}

	if (!FPaths::DirectoryExists(Options.InstallDir))
	{
		UE_LOGF(LogUninstallHelper, Error, "InstallDir %ls does not exist. Exiting.", *Options.InstallDir);
		return static_cast<int32>(UninstallHelper::EReturnCode::InvalidInstallDir);
	}
	FParse::Bool(*CmdLine, TEXT("AllowHarshReturnCodes="), Options.bAllowHarshReturnCodes);
	FParse::Bool(*CmdLine, TEXT("DeletePersistentDownloadDir="), Options.bDeletePersistentDownloadDir);
	FParse::Bool(*CmdLine, TEXT("DeleteConfig="), Options.bDeleteConfig);
	Options.bShouldExecute = !FParse::Param(*CmdLine, TEXT("NoExecute")); // An option to just log with this application, do not actually do anything

	FString ListDelimiter = ";";
	FParse::Value(*CmdLine, TEXT("ArtifactDelimiter="), ListDelimiter);
	
	FString DeleteDirsCmd;
	if (FParse::Value(*CmdLine, TEXT("DeleteArtifacts="), DeleteDirsCmd))
	{
		DeleteDirsCmd.ParseIntoArray(Options.AdditionalDeleteArtifacts, *ListDelimiter);
	}

	FString UninstallArtifactsManifestFilename;
	if (FParse::Value(*CmdLine, TEXT("UninstallArtifactsManifest="), UninstallArtifactsManifestFilename))
	{
		TArray<FString> UninstallArtifactsManifest;
		const FString ManifestPath = UninstallHelper::ReplacePathTokens(UninstallArtifactsManifestFilename, Options);
		if (!FFileHelper::LoadFileToStringArray(UninstallArtifactsManifest, *ManifestPath))
		{
			UE_LOGF(LogUninstallHelper, Warning, "Failed to find uninstall artifacts manifest file '%ls', skipping.", *ManifestPath);
		}
		Options.AdditionalDeleteArtifacts.Append(UninstallArtifactsManifest);
	}

	FString UninstallActionsManifestFilename;
	if (FParse::Value(*CmdLine, TEXT("UninstallActionsManifest="), UninstallActionsManifestFilename))
	{
		TArray<FString> UninstallActionsManifest;
		const FString ManifestPath = UninstallHelper::ReplacePathTokens(UninstallActionsManifestFilename, Options);
		if (FFileHelper::LoadFileToStringArray(UninstallActionsManifest, *ManifestPath))
		{
			for (const FString& Action : UninstallActionsManifest)
			{
				FString Url;
				FString Args;
				if (Action.StartsWith(TEXT("\"")))
				{
					int Size;
					FParse::QuotedString(*Action, Url, &Size);
					Url = Url.TrimQuotes();
					Args = Action.RightChop(Size).TrimStart();
				}
				else
				{
					Action.Split(TEXT(" "), &Url, &Args);
				}
				Url = ReplacePathTokens(Url, Options);
				Args = ReplacePathTokens(Args, Options);
				Options.AdditionalActions.Add({ Url, Args });
			}
		}
		else
		{
			UE_LOGF(LogUninstallHelper, Warning, "Failed to find uninstall actions manifest file '%ls', skipping.", *ManifestPath);
		}
	}

	const UninstallHelper::EReturnCode Result = UninstallHelper::Execute(Options);
	if (Result != UninstallHelper::EReturnCode::Success)
	{
		UE_LOGF(LogUninstallHelper, Error, "UninstallHelper failed to execute with exit code %d", static_cast<int32>(Result));
	}
	return Options.bAllowHarshReturnCodes ? static_cast<int32>(Result) : 0;
}
