// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogUninstallHelper, Log, All);

namespace UninstallHelper
{
	enum class EReturnCode : int32
	{
		Success = 0,
		UnknownError,
		ArgumentError,
		UnknownAppName,
		InvalidInstallDir,
		ExecActionFailed,
		DiskOperationFailed,

		Crash = 255
	};
}