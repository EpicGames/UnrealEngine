// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UninstallHelper : ModuleRules
{
	public UninstallHelper(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePaths.Add("Runtime/Launch/Public");
		PrivateIncludePaths.Add("Runtime/Launch/Private");
		
		PrivateDependencyModuleNames.AddRange(new []
		{
			"Core", "Projects"
		});
	}
}
